{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Live Update for Slate",
	"Description": "Refreshes the editor layout and tabs when Live Coding is complete",
	"Category": "Other",
	"CreatedBy": "Alessa Baker & Epic Games, Inc.",
	"CreatedByURL": "codekitten.me",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "LiveUpdateForSlate",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}