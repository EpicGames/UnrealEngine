// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveUpdateForSlate : ModuleRules
{
	public LiveUpdateForSlate(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"MainFrame",
				"Settings",
				"SettingsEditor",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

		if (Target.bWithLiveCoding)
		{
			PrivateDependencyModuleNames.Add("LiveCoding");
		}
	}
}
