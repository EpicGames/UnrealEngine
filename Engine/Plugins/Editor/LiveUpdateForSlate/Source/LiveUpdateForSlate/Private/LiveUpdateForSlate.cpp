// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveUpdateForSlate.h"
#include "Editor.h"
#include "Framework/Docking/TabManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "ISettingsModule.h"
#include "LiveUpdateSlateSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Subsystems/AssetEditorSubsystem.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif

#define LOCTEXT_NAMESPACE "LiveUpdateForSlateModule"

IMPLEMENT_MODULE(FLiveUpdateForSlateModule, LiveUpdateForSlate)

void FLiveUpdateForSlateModule::StartupModule()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Editor", "Plugins", "Live Update for Slate",
			LOCTEXT("LiveUpdateSlateSettingsName", "Live Update for Slate"),
			LOCTEXT("LiveUpdateSlateSettingsDescription", "Live Update for Slate"),
			GetMutableDefault<ULiveUpdateSlateSettings>()
		);
	}

#if WITH_LIVE_CODING
	if (ILiveCodingModule* LiveCoding = FModuleManager::LoadModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME))
	{
		OnPatchCompleteHandle = LiveCoding->GetOnPatchCompleteDelegate().AddRaw(this, &FLiveUpdateForSlateModule::OnPatchComplete);
	}
#endif
}

void FLiveUpdateForSlateModule::ShutdownModule()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "Plugins", "Live Update for Slate");
	}

#if WITH_LIVE_CODING
	if (ILiveCodingModule* LiveCoding = FModuleManager::LoadModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME))
	{
		LiveCoding->GetOnPatchCompleteDelegate().Remove(OnPatchCompleteHandle);
	}
#endif
}

void FLiveUpdateForSlateModule::OnPatchComplete()
{
	ULiveUpdateSlateSettings* Settings = GetMutableDefault<ULiveUpdateSlateSettings>();

	// Don't rebuild Slate if the plugin is not enabled
	if (!Settings->bEnableLiveUpdateForSlate)
	{
		return;
	}

	// Save all open tabs
	FGlobalTabmanager::Get()->SaveAllVisualState();

	// Flush our config files
	const bool bRead = true;
	GConfig->Flush(bRead, GEditorLayoutIni);

	UAssetEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> OpenedAssets = Subsystem->GetAllEditedAssets();

	// Request the editor to rebuild Slate
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	MainFrameModule.RecreateDefaultMainFrame(false, false);

	for (const UObject* Asset : OpenedAssets)
	{
		Subsystem->OpenEditorForAsset(Asset);
	}
}

#undef LOCTEXT_NAMESPACE
