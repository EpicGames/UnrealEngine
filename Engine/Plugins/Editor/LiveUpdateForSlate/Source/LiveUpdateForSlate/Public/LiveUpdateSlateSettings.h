// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "LiveUpdateSlateSettings.generated.h"

UCLASS(config = EditorPerProjectUserSettings)
class LIVEUPDATEFORSLATE_API ULiveUpdateSlateSettings : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Config, EditAnywhere, Category=Slate, meta=(ToolTip="Refreshes the editor's Slate layout when Live Coding patches are complete."))
	bool bEnableLiveUpdateForSlate = true;
};
